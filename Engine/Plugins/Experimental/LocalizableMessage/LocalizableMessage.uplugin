{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Localizable Message",
	"Description": "Utility for a text message that can be replicated. It supports parameter resolution for the client.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LocalizableMessage",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LocalizableMessageBlueprint",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}
