// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LocalizableMessage : ModuleRules
	{
		public LocalizableMessage(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
				});
		}
	}
}
