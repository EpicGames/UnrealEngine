// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizableMessage.h"
#include "Internationalization/TextNamespaceUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalizableMessage)

namespace UE::LocalizableMessageTextInterop
{
	bool TextIdToMessageKey(const FTextId& TextId, FString& OutMessageKey)
	{
		if (TextId.IsEmpty())
		{
			return false;
		}

		FString Namespace = TextId.GetNamespace().ToString();
#if USE_STABLE_LOCALIZATION_KEYS
		// Remove any package ID from the namespace, as we only translate the "clean" version of the namespace
		Namespace = TextNamespaceUtil::StripPackageNamespace(Namespace);
#endif	// USE_STABLE_LOCALIZATION_KEYS

		if (Namespace.IsEmpty())
		{
			// If there is no namespace then we can just use the key directly
			TextId.GetKey().ToString(OutMessageKey);
			return true;
		}

		// If there is a namespace then we need to combine the namespace+key together
		Namespace.ReplaceInline(TEXT(","), TEXT("\\,"));

		FString Key = TextId.GetKey().ToString();
		Key.ReplaceInline(TEXT(","), TEXT("\\,"));

		// Then we use a leading ~ character to denote this combination has happened
		OutMessageKey = FString::Printf(TEXT("~%s,%s"), *Namespace, *Key);
		return true;
	}

	bool MessageKeyToTextId(const FString& MessageKey, FTextId& OutTextId)
	{
		if (MessageKey.IsEmpty())
		{
			return false;
		}

		// This might be a combined namespace+key pair
		if (MessageKey[0] == TEXT('~'))
		{
			FString Namespace;
			FString Key;
			if (MessageKey.Split(TEXT(","), &Namespace, &Key))
			{
				Namespace.RemoveAt(1, EAllowShrinking::No); // Remove the ~ marker denoting that this was a combined namespace+key pair
				Namespace.ReplaceInline(TEXT("\\,"), TEXT(","));
				Key.ReplaceInline(TEXT("\\,"), TEXT(","));
				OutTextId = FTextId(Namespace, Key);
				return true;
			}
			// Note: We don't return false if the Split failed, as it's possible (though unlikely) that this was a non-combined key that happened to start with a ~
		}

		OutTextId = FTextId(FTextKey(), MessageKey);
		return true;
	}
}

FLocalizableMessageParameterEntry::FLocalizableMessageParameterEntry() = default;
FLocalizableMessageParameterEntry::FLocalizableMessageParameterEntry(const FString& InKey, const FInstancedStruct& InValue) : Key(InKey), Value(InValue) {}
FLocalizableMessageParameterEntry::~FLocalizableMessageParameterEntry() = default;

bool FLocalizableMessageParameterEntry::operator==(const FLocalizableMessageParameterEntry& Other) const
{
	return Key == Other.Key &&
		   Value == Other.Value;
}

FLocalizableMessage::FLocalizableMessage() = default;
FLocalizableMessage::~FLocalizableMessage() = default;

bool FLocalizableMessage::operator==(const FLocalizableMessage& Other) const
{
	return Key == Other.Key &&
		DefaultText == Other.DefaultText &&
		Substitutions == Other.Substitutions;
}
