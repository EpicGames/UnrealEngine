// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILocalizableMessageModule.h"
#include "LocalizableMessageProcessor.h"
#include "LocalizableMessageTypes.h"
#include "Modules/ModuleManager.h"

class FLocalizableMessageModule : public ILocalizableMessageModule
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual FLocalizableMessageProcessor& GetLocalizableMessageProcessor() override { return LocalizableMessageProcessor; }

private:

	FLocalizableMessageProcessor LocalizableMessageProcessor;
};

void FLocalizableMessageModule::StartupModule()
{
	LocalizableMessageTypes::RegisterTypes();
}

void FLocalizableMessageModule::ShutdownModule()
{
	LocalizableMessageTypes::UnregisterTypes();
}

ILocalizableMessageModule& ILocalizableMessageModule::Get()
{
	ILocalizableMessageModule* VerseModule;
	UE_AUTORTFM_OPEN{ VerseModule = &FModuleManager::LoadModuleChecked<FLocalizableMessageModule>(TEXT("LocalizableMessage")); };
	return *VerseModule;
}

IMPLEMENT_MODULE(FLocalizableMessageModule, LocalizableMessage)
