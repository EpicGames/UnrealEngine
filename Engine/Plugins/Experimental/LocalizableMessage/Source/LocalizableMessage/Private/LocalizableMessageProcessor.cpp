// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizableMessageProcessor.h"

#include "HAL/IConsoleManager.h"
#include "Internationalization/Text.h"
#include "LocalizableMessage.h"
#include "LocalizationContext.h"

DEFINE_LOG_CATEGORY(LogLocalizableMessageProcessor);

namespace LocalizableMessageProcessor
{

// Disable argument modifier evaluation for text created from a message
bool bAllowTextArgumentModifiers = false;
FAutoConsoleVariableRef CVarAllowTextArgumentModifiers(TEXT("Localization.Message.AllowTextArgumentModifiers"), bAllowTextArgumentModifiers, TEXT("Whether to allow message -> text conversion to use text-style argument modifiers (default: false)"));

ETextFormatFlags GetTextFormatFlags()
{
	return bAllowTextArgumentModifiers
		? ETextFormatFlags::Default
		: (ETextFormatFlags::Default & ~ETextFormatFlags::EvaluateArgumentModifiers);
}

}

FLocalizableMessageProcessor::FLocalizableMessageProcessor()
{

}

FLocalizableMessageProcessor::~FLocalizableMessageProcessor()
{
	ensure(LocalizeValueMapping.IsEmpty());
}

FText FLocalizableMessageProcessor::Localize(const FLocalizableMessage& Message, const FLocalizationContext& Context)
{
	FFormatNamedArguments FormatArguments;
	for (const FLocalizableMessageParameterEntry& Substitution : Message.Substitutions)
	{
		if (ensure(Substitution.Value.IsValid()))
		{
			if (const LocalizeValueFnc* Functor = LocalizeValueMapping.Find(Substitution.Value.GetScriptStruct()->GetFName()))
			{
				FFormatArgumentValue SubstitutionResult = (*Functor)(Substitution.Value, Context);
				FormatArguments.Add(Substitution.Key, MoveTemp(SubstitutionResult));
			}
			else
			{
				ensureMsgf(false, TEXT("Localization type %s not registered in Localization Processor."), *Substitution.Value.GetScriptStruct()->GetFName().ToString());
			}
		}
		else
		{
			ensureMsgf(false, TEXT("Message contained null substitution."));
		}
	}

	// an unfortunate number of allocations and copies here
	FText LocalizedText;
	if (!Message.IsEmpty())
	{
		FTextId TextId;
		if (UE::LocalizableMessageTextInterop::MessageKeyToTextId(Message.Key, TextId))
		{
			LocalizedText = FText::AsLocalizable_Advanced(TextId.GetNamespace(), TextId.GetKey(), Message.DefaultText);
		}
		else
		{
			LocalizedText = FText::AsCultureInvariant(Message.DefaultText);
		}
	}

	if (FormatArguments.Num() > 0)
	{
		const bool bIsLocalized = !LocalizedText.ToString().Equals(Message.DefaultText);
		FTextFormat LocalizedTextFormat(LocalizedText, LocalizableMessageProcessor::GetTextFormatFlags() | (bIsLocalized ? ETextFormatFlags::EvaluateArgumentModifiers : ETextFormatFlags::None));
		LocalizedText = FText::Format(LocalizedTextFormat, FormatArguments);
	}

	UE_LOGF(LogLocalizableMessageProcessor, VeryVerbose, "Localized Text: [%ls]", *LocalizedText.ToString());

	return LocalizedText;
}

void FLocalizableMessageProcessor::UnregisterLocalizableTypes(FScopedRegistrations& ScopedRegistrations)
{
	for (const FName& Registration : ScopedRegistrations.Registrations)
	{
		LocalizeValueMapping.Remove(Registration);
	}

	ScopedRegistrations.Registrations.Reset();
}
