// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizableMessageTypes.h"

#include "ILocalizableMessageModule.h"
#include "LocalizableMessageBaseParameters.h"
#include "LocalizableMessageProcessor.h"
#include "LocalizationContext.h"

namespace LocalizableMessageTypes
{
	FLocalizableMessageProcessor::FScopedRegistrations RegisteredLocalizationTypes;

	FFormatArgumentValue Int_LocalizeValue(const FLocalizableMessageParameterInt& Localizable, const FLocalizationContext& LocalizationContext)
	{
		FCulturePtr LocaleOverride = LocalizationContext.GetLocaleOverride();
		return LocaleOverride
			? FFormatArgumentValue(FText::AsNumber(Localizable.Value, nullptr, LocalizationContext.GetLocaleOverride()))
			: FFormatArgumentValue(Localizable.Value);
	}
	FFormatArgumentValue Float_LocalizeValue(const FLocalizableMessageParameterFloat& Localizable, const FLocalizationContext& LocalizationContext)
	{
		FCulturePtr LocaleOverride = LocalizationContext.GetLocaleOverride();
		return LocaleOverride
			? FFormatArgumentValue(FText::AsNumber(Localizable.Value, nullptr, LocalizationContext.GetLocaleOverride()))
			: FFormatArgumentValue(Localizable.Value);
	}
	FFormatArgumentValue String_LocalizeValue(const FLocalizableMessageParameterString& Localizable, const FLocalizationContext& LocalizationContext)
	{
		return FText::AsCultureInvariant(Localizable.Value);
	}
	FFormatArgumentValue Message_LocalizeValue(const FLocalizableMessageParameterMessage& Localizable, const FLocalizationContext& LocalizationContext)
	{
		ILocalizableMessageModule& LocalizableMessageModule = ILocalizableMessageModule::Get();
		FLocalizableMessageProcessor& Processor = LocalizableMessageModule.GetLocalizableMessageProcessor();

		return Processor.Localize(Localizable.Value, LocalizationContext);
	}


	void RegisterTypes()
	{
		ILocalizableMessageModule& LocalizableMessageModule = ILocalizableMessageModule::Get();
		FLocalizableMessageProcessor& Processor = LocalizableMessageModule.GetLocalizableMessageProcessor();

		Processor.RegisterLocalizableType<FLocalizableMessageParameterInt>(&Int_LocalizeValue,RegisteredLocalizationTypes);
		Processor.RegisterLocalizableType<FLocalizableMessageParameterFloat>(&Float_LocalizeValue,RegisteredLocalizationTypes);
		Processor.RegisterLocalizableType<FLocalizableMessageParameterString>(&String_LocalizeValue, RegisteredLocalizationTypes);
		Processor.RegisterLocalizableType<FLocalizableMessageParameterMessage>(&Message_LocalizeValue, RegisteredLocalizationTypes);
	}

	void UnregisterTypes()
	{
		ILocalizableMessageModule& LocalizableMessageModule = ILocalizableMessageModule::Get();
		FLocalizableMessageProcessor& Processor = LocalizableMessageModule.GetLocalizableMessageProcessor();

		Processor.UnregisterLocalizableTypes(RegisteredLocalizationTypes);
	}
}