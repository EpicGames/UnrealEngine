// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#define UE_API LOCALIZABLEMESSAGE_API

class FLocalizableMessageProcessor;

class ILocalizableMessageModule : public IModuleInterface
{
public:
	static UE_API ILocalizableMessageModule& Get();

	virtual FLocalizableMessageProcessor& GetLocalizableMessageProcessor() = 0;
};

#undef UE_API
