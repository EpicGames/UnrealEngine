// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/UniquePtr.h"

#include "LocalizableMessage.generated.h"

#define UE_API LOCALIZABLEMESSAGE_API

namespace UE::LocalizableMessageTextInterop
{
	/**
	 * Attempt to convert the given FText ID to a string that can be used as a FLocalizableMessage key.
	 * @return True if the conversion was possible, or false if it failed.
	 */
	LOCALIZABLEMESSAGE_API bool TextIdToMessageKey(const FTextId& TextId, FString& OutMessageKey);

	/**
	 * Attempt to convert the given FLocalizableMessage key into a FText ID.
	 * @return True if the conversion was possible, or false if it failed.
	 */
	LOCALIZABLEMESSAGE_API bool MessageKeyToTextId(const FString& MessageKey, FTextId& OutTextId);
}


USTRUCT()
struct FLocalizableMessageParameterEntry
{
	GENERATED_BODY()

public:

	UE_API FLocalizableMessageParameterEntry();
	UE_API FLocalizableMessageParameterEntry(const FString& InKey, const FInstancedStruct& InValue);
	UE_API ~FLocalizableMessageParameterEntry();

	UE_API bool operator==(const FLocalizableMessageParameterEntry& Other) const;
	bool operator!=(const FLocalizableMessageParameterEntry& Other) const
	{
		return !(*this == Other);
	}

	UPROPERTY()
	FString Key;

	UPROPERTY()
	FInstancedStruct Value;
};

USTRUCT(BlueprintType)
struct FLocalizableMessage
{
	GENERATED_BODY();

	UE_API FLocalizableMessage();
	UE_API ~FLocalizableMessage();
	UE_API bool operator==(const FLocalizableMessage& Other) const;

	void Reset()
	{
		Key.Reset();
		DefaultText.Reset();
		Substitutions.Reset();
	}

	bool operator!=(const FLocalizableMessage& Other) const
	{
		return !(*this == Other);
	}

	bool IsEmpty() const
	{
		return Key.IsEmpty() && DefaultText.IsEmpty();
	}

	UPROPERTY()
	FString Key;

	UPROPERTY()
	FString DefaultText;

	UPROPERTY()
	TArray<FLocalizableMessageParameterEntry> Substitutions;
};

#undef UE_API
