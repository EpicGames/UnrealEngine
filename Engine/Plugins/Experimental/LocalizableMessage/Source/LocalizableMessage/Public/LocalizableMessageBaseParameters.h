// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "LocalizableMessage.h"

#include "LocalizableMessageBaseParameters.generated.h"

USTRUCT()
struct FLocalizableMessageParameterInt
{
	GENERATED_BODY()

public:

	UPROPERTY()
	int64 Value = 0;
};

USTRUCT()
struct FLocalizableMessageParameterFloat
{
	GENERATED_BODY()

public:

	UPROPERTY()
	double Value = 0;
};

USTRUCT()
struct FLocalizableMessageParameterString
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FString Value;
};

USTRUCT()
struct FLocalizableMessageParameterMessage
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FLocalizableMessage Value;
};