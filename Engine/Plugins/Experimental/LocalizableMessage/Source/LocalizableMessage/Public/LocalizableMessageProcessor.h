// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "StructUtils/InstancedStruct.h"
#include "Logging/LogMacros.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"

class FText;
class FFormatArgumentValue;
struct FLocalizableMessage;
struct FLocalizationContext;

class FLocalizableMessageProcessor
{
public:

	struct FScopedRegistrations
	{
		FScopedRegistrations() = default;
		~FScopedRegistrations() { ensure(Registrations.Num() == 0); }

		TArray<FName> Registrations;
	};

	FLocalizableMessageProcessor();
	~FLocalizableMessageProcessor();

	LOCALIZABLEMESSAGE_API FText Localize(const FLocalizableMessage& Message, const FLocalizationContext& Context);

	template <typename UserType>
	void RegisterLocalizableType(const TFunction<FFormatArgumentValue(const UserType&, const FLocalizationContext&)>& LocalizeValueFunctor, FScopedRegistrations& ScopedRegistrations)
	{
		auto FncLocalizeValue = [LocalizeValueFunctor](const FInstancedStruct& Localizable, const FLocalizationContext& LocalizationContext)
		{
			return LocalizeValueFunctor(Localizable.Get<UserType>(), LocalizationContext);
		};

		FName UserId = UserType::StaticStruct()->GetFName();
		LocalizeValueMapping.Add(UserId, FncLocalizeValue);
		ScopedRegistrations.Registrations.Add(UserId);
	}

	LOCALIZABLEMESSAGE_API void UnregisterLocalizableTypes(FScopedRegistrations& ScopedRegistrations);

private:
	using LocalizeValueFnc = TFunction<FFormatArgumentValue(const FInstancedStruct&, const FLocalizationContext&)>;

	TMap<FName, LocalizeValueFnc> LocalizeValueMapping;
};

DECLARE_LOG_CATEGORY_EXTERN(LogLocalizableMessageProcessor, Log, All);