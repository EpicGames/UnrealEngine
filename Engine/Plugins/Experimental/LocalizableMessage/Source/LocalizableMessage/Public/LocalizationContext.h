// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/CulturePointer.h"
#include "UObject/WeakObjectPtr.h"

struct FLocalizationContext
{
	FLocalizationContext() = default;
	explicit FLocalizationContext(const UObject* InWorldContext, FCulturePtr InLanguageOverride = nullptr, FCulturePtr InLocaleOverride = nullptr)
		: WorldContext(InWorldContext)
		, LanguageOverride(MoveTemp(InLanguageOverride))
		, LocaleOverride(MoveTemp(InLocaleOverride))
	{}

	TWeakObjectPtr<const UObject> WorldContext = nullptr;

	FCulturePtr GetLanguageOverride() const
	{
		return LanguageOverride;
	}

	FCulturePtr GetLocaleOverride() const
	{
		return LocaleOverride
			? LocaleOverride
			: LanguageOverride;
	}

private:
	/**
	 * The current language (for localization) override (if any).
	 * @note Will also be used as the locale if LocaleOverride is null.
	 * @note UE doesn't currently support loading the localization data for multiple languages at the same time.
	 */
	FCulturePtr LanguageOverride;

	/**
	 * The current locale (for internationalization) override (if any).
	 * @note Internationalization affects things like number, date, time, etc formatting.
	 */
	FCulturePtr LocaleOverride;
};

