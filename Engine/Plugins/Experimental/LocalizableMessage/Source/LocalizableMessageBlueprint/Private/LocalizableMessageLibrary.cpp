// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizableMessageLibrary.h"

#include "ILocalizableMessageModule.h"
#include "Internationalization/Internationalization.h"
#include "LocalizableMessage.h"
#include "LocalizableMessageProcessor.h"
#include "LocalizationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalizableMessageLibrary)

FText ULocalizableMessageLibrary::Conv_LocalizableMessageToText(UObject* WorldContextObject, const FLocalizableMessage& Message)
{
	FLocalizationContext LocContext(WorldContextObject);
	FLocalizableMessageProcessor& Processor = ILocalizableMessageModule::Get().GetLocalizableMessageProcessor();
	return Processor.Localize(Message, LocContext);
}

bool ULocalizableMessageLibrary::EqualEqual_LocalizableMessage(const FLocalizableMessage& A, const FLocalizableMessage& B)
{
	return A == B;
}

bool ULocalizableMessageLibrary::IsEmpty_LocalizableMessage(const FLocalizableMessage& Message)
{
	return Message.IsEmpty();
}

void ULocalizableMessageLibrary::Reset_LocalizableMessage(FLocalizableMessage& Message)
{
	Message.Reset();
}