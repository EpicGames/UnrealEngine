// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "LocalizableMessageLibrary.generated.h"

struct FLocalizableMessage;
class FText;

/** BlueprintFunctionLibrary for LocalizableMessage */
UCLASS(MinimalAPI)
class ULocalizableMessageLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Conversion function from LocalizableMessage to FText.
	 */
	UFUNCTION(BlueprintPure, Category = "Localizable Message", meta = (WorldContext = "WorldContextObject"))
	static LOCALIZABLEMESSAGEBLUEPRINT_API FText Conv_LocalizableMessageToText(UObject* WorldContextObject, UPARAM(ref) const FLocalizableMessage& Message);

	/**
	 * Returns true if the Localizable Messages are equal (A == B)
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Localizable Message", meta = (DisplayName = "Equal (LocalizableMessage)", CompactNodeTitle = "==", BlueprintThreadSafe))
	static LOCALIZABLEMESSAGEBLUEPRINT_API bool EqualEqual_LocalizableMessage(UPARAM(ref) const FLocalizableMessage& A, UPARAM(ref) const FLocalizableMessage& B);

	/**
	 * Returns true if the message is empty
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Localizable Message")
	static LOCALIZABLEMESSAGEBLUEPRINT_API bool IsEmpty_LocalizableMessage(UPARAM(ref) const FLocalizableMessage& Message);

	/**
	 * Resets the Localizable Message
	 */
	UFUNCTION(BlueprintCallable, Category = "Localizable Message")
	static LOCALIZABLEMESSAGEBLUEPRINT_API void Reset_LocalizableMessage(UPARAM(ref) FLocalizableMessage& Message);
};

